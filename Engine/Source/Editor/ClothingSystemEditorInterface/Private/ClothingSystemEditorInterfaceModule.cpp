// Copyright Epic Games, Inc. All Rights Reserved.

#include "ClothingSystemEditorInterfaceModule.h"

#include "ClothingAssetFactoryInterface.h"
#include "Containers/Array.h"
#include "Features/IModularFeatures.h"
#include "HAL/IConsoleManager.h"
#include "HAL/Platform.h"
#include "Modules/ModuleManager.h"
#include "SimulationEditorExtender.h"
#include "UObject/Class.h"

IMPLEMENT_MODULE(FClothingSystemEditorInterfaceModule, ClothingSystemEditorInterface);


namespace UE::Private
{
	static FString DefaultClothingAssetFactory = TEXT("ClothingAssetFactory");  // UClothingAssetFactory is the default provider 

	static TAutoConsoleVariable<FString> CVarDefaultClothingAssetFactoryClass(
		TEXT("p.Cloth.DefaultClothingAssetFactoryClass"),
		DefaultClothingAssetFactory,
		TEXT("The class name of the default clothing asset factory."),
		ECVF_Default);
}


const FName FClothingSystemEditorInterfaceModule::ExtenderFeatureName(TEXT("ClothingSimulationEditorExtender"));

FClothingSystemEditorInterfaceModule::FClothingSystemEditorInterfaceModule()
{

}

void FClothingSystemEditorInterfaceModule::StartupModule()
{

}

void FClothingSystemEditorInterfaceModule::ShutdownModule()
{

}

UClothingAssetFactoryBase* FClothingSystemEditorInterfaceModule::GetClothingAssetFactory()
{
	TArray<IClothingAssetFactoryProvider*> Providers = IModularFeatures::Get().GetModularFeatureImplementations<IClothingAssetFactoryProvider>(IClothingAssetFactoryProvider::FeatureName);

	UClothingAssetFactoryBase* ClothingAssetFactory = nullptr;
	for (IClothingAssetFactoryProvider* const Provider : Providers)
	{
		if (Provider)
		{
			if (UClothingAssetFactoryBase* const Factory = Provider->GetFactory())
			{
				if (Factory->GetClass()->GetName() == UE::Private::CVarDefaultClothingAssetFactoryClass.GetValueOnAnyThread())
				{
					ClothingAssetFactory = Factory;
				}
			}
		}
	}
	return ClothingAssetFactory;
}

TArray<UClothingAssetFactoryBase*> FClothingSystemEditorInterfaceModule::GetClothingAssetFactories()
{
	const TArray<IClothingAssetFactoryProvider*> Providers = IModularFeatures::Get().GetModularFeatureImplementations<IClothingAssetFactoryProvider>(IClothingAssetFactoryProvider::FeatureName);

	TArray<UClothingAssetFactoryBase*> ClothingAssetFactories;
	for (IClothingAssetFactoryProvider* const Provider : Providers)
	{
		if (Provider)
		{
			if (UClothingAssetFactoryBase* const Factory = Provider->GetFactory())
			{
				ClothingAssetFactories.Emplace(Factory);
			}
		}
	}
	return ClothingAssetFactories;
}

ISimulationEditorExtender* FClothingSystemEditorInterfaceModule::GetSimulationEditorExtender(FName InSimulationClassName)
{
	TArray<ISimulationEditorExtender*> Extenders = IModularFeatures::Get().GetModularFeatureImplementations<ISimulationEditorExtender>(ExtenderFeatureName);

	for(ISimulationEditorExtender* Extender : Extenders)
	{
		UClass* SupportedClass = Extender->GetSupportedSimulationFactoryClass();

		if(SupportedClass && SupportedClass->GetFName() == InSimulationClassName)
		{
			return Extender;
		}
	}

	return nullptr;
}
