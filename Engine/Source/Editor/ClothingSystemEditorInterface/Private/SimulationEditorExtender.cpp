// Copyright Epic Games, Inc. All Rights Reserved.

#include "SimulationEditorExtender.h"
#include "ClothingSimulationInterface.h"

void ISimulationEditorExtender::DebugDrawSimulation(const IClothingSimulationInterface* InSimulation, USkeletalMeshComponent* InOwnerComponent, FPrimitiveDrawInterface* PDI)
{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	if (ensureMsgf(InSimulation && InSimulation->DynamicCastToIClothingSimulation(), TEXT("DebugDrawSimulation(const IClothingSimulationInterface*, ...) must be implemented from 5.7, as the function will become pure virtual.")))
	{
		DebugDrawSimulation(InSimulation->DynamicCastToIClothingSimulation(), InOwnerComponent, PDI);
	}
PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

void ISimulationEditorExtender::DebugDrawSimulationTexts(const IClothingSimulationInterface* InSimulation, USkeletalMeshComponent* InOwnerComponent, FCanvas* Canvas, const FSceneView* SceneView)
{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	if (ensureMsgf(InSimulation && InSimulation->DynamicCastToIClothingSimulation(), TEXT("DebugDrawSimulationTexts(const IClothingSimulationInterface*, ...) must be implemented from 5.7, as the function will become pure virtual.")))
	{
		DebugDrawSimulationTexts(InSimulation->DynamicCastToIClothingSimulation(), InOwnerComponent, Canvas, SceneView);
	}
PRAGMA_ENABLE_DEPRECATION_WARNINGS
}
