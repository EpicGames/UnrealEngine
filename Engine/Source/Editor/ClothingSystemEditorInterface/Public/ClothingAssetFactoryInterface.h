// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/UnrealString.h"
#include "CoreTypes.h"
#include "Features/IModularFeature.h"
#include "Misc/AssertionMacros.h"
#include "UObject/NameTypes.h"
#include "UObject/Object.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UObjectGlobals.h"
#include "UObject/UnrealNames.h"

#include "ClothingAssetFactoryInterface.generated.h"

#define UE_API CLOTHINGSYSTEMEDITORINTERFACE_API

namespace nvidia
{
	namespace apex
	{
		class ClothingAsset;
	}
}

class UClothingAssetBase;
class USkeletalMesh;
struct FSkeletalMeshClothBuildParams;

// Clothing asset factories should inherit this interface/uobject to provide functionality
// to build clothing assets from .apx files imported to the engine
UCLASS(MinimalAPI, Abstract)
class UClothingAssetFactoryBase : public UObject
{
	GENERATED_BODY()

public:

	/** Return the supported source asset type for this clothing data. @see CreateFromSourceAsset. */
	virtual const UClass* GetSupportedSourceAssetType() const
	PURE_VIRTUAL(UClothingAssetFactoryBase::GetSupportedSourceAssetType, return nullptr;);

	/**
	 * Given a target mesh and parameters describing the build operation, create a clothing asset for
	 * use on the mesh
	 * @param TargetMesh - The mesh to target
	 * @param Params - Extra operation params (LOD/Section index etc...)
	 */
	virtual UClothingAssetBase* CreateFromSkeletalMesh(USkeletalMesh* TargetMesh, FSkeletalMeshClothBuildParams& Params)
	PURE_VIRTUAL(UClothingAssetFactoryBase::CreateFromSkeletalMesh, return nullptr;);

	/**
	 * Given a target mesh and a source asset, create clothing data from the source asset to use on the target mesh.
	 * @param TargetMesh - The mesh to target
	 * @param SourceAsset - The asset to build the clothing data with.
	 */
	virtual TArray<UClothingAssetBase*> CreateFromSourceAsset(USkeletalMesh* TargetMesh, const UObject* SourceAsset) const
	PURE_VIRTUAL(UClothingAssetFactoryBase::CreateFromSourceAsset, return TArray<UClothingAssetBase*>(););

	/** 
	 * Given a target mesh and valid parameters, import a simulation mesh as a LOD for the clothing
	 * specified by the build parameters, returning the modified clothing object
	 * @param TargetMesh The owner mesh
	 * @param Params Build parameters for the operation (target clothing object, source data)
	 */
	virtual UClothingAssetBase* ImportLodToClothing(USkeletalMesh* TargetMesh, FSkeletalMeshClothBuildParams& Params)
	PURE_VIRTUAL(UClothingAssetFactoryBase::ImportLodToClothing, return nullptr;);

	// Deprecated methods
	UE_DEPRECATED(5.7, "Clothing file import is no longer supported.")
	virtual UClothingAssetBase* Import(const FString& Filename, USkeletalMesh* TargetMesh, FName InName = NAME_None)
	{
		return nullptr;
	}

	UE_DEPRECATED(5.7, "Clothing file import is no longer supported.")
	virtual UClothingAssetBase* Reimport(const FString& Filename, USkeletalMesh* TargetMesh, UClothingAssetBase* OriginalAsset)
	{
		return nullptr;
	}

	UE_DEPRECATED(5.7, "Use UClothingAssetFactory::CreateFromSourceAsset() instead.")
	virtual UClothingAssetBase* CreateFromExistingCloth(USkeletalMesh* /*TargetMesh*/, USkeletalMesh* /*SourceMesh*/, UClothingAssetBase* /*SourceAsset*/)
	{
		return nullptr;
	}

	UE_DEPRECATED(5.7, "Clothing file import is no longer supported.")
	virtual bool CanImport(const FString& Filename)
	{
		return false;
	}

	UE_DEPRECATED(5.7, "Clothing file import is no longer supported.")
	virtual UClothingAssetBase* CreateFromApexAsset(nvidia::apex::ClothingAsset* InApexAsset, USkeletalMesh* TargetMesh, FName InName = NAME_None)
	{
		return nullptr;
	}
};

// An interface for a class that will provide a clothing asset factory, this should be
// registered as a feature under its FeatureName to be picked up by the engine
class IClothingAssetFactoryProvider : public IModularFeature
{
public:
	static UE_API const FName FeatureName;

	// Called by the engine to retrieve a valid factory from a provider
	// This can be the default object for the factory class or a full instance
	virtual UClothingAssetFactoryBase* GetFactory() = 0;
};

#undef UE_API
