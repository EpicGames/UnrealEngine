// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Modules/ModuleInterface.h"
#include "UObject/NameTypes.h"

#define UE_API CLOTHINGSYSTEMEDITORINTERFACE_API

class ISimulationEditorExtender;
class UClothingAssetFactoryBase;

class FClothingSystemEditorInterfaceModule : public IModuleInterface
{

public:

	UE_API const static FName ExtenderFeatureName;

	UE_API FClothingSystemEditorInterfaceModule();

	UE_API virtual void StartupModule() override;
	UE_API virtual void ShutdownModule() override;

	UE_API UClothingAssetFactoryBase* GetClothingAssetFactory();
	UE_API TArray<UClothingAssetFactoryBase*> GetClothingAssetFactories();
	UE_API ISimulationEditorExtender* GetSimulationEditorExtender(FName InSimulationFactoryClassName);

private:

};

#undef UE_API
