// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Features/IModularFeature.h"
#include "Templates/SharedPointer.h"

#define UE_API CLOTHINGSYSTEMEDITORINTERFACE_API

#if UE_ENABLE_INCLUDE_ORDER_DEPRECATED_IN_5_7
class UE_DEPRECATED(5.7, "Use IClothingSimulationInterface instead.") IClothingSimulation;
#endif

class UClass;
class FMenuBuilder;
class IClothingSimulationInterface;
class USkeletalMeshComponent;
class IPersonaPreviewScene;
class FPrimitiveDrawInterface;
class FCanvas;
class FSceneView;

/**
 * A simulation extender is an object responsible for extending certain editor features with respect to a certain
 * type of clothing simulation. The supported simulation factory class should be returned from GetSupportedSimulationFactoryClass.
 * The engine will call into various interface functions to perform editor functions as outlined in the interface API
 * As this is a modular feature it should be registered under FClothingSystemEditorInterfaceModule::ExtenderFeatureName
 * To be accessible through the clothing editor interface module.
 */
class ISimulationEditorExtender : public IModularFeature
{

public:
	virtual ~ISimulationEditorExtender() = default;

	/**
	 * Called to identify the type of clothing simulation this editor extender can support.
	 * Should return a class derived from UClothingSimulationFactory
	 */
	virtual UClass* GetSupportedSimulationFactoryClass() = 0;

	/**
	 * Called from the editor to add simulation specific entries to the "Show" menu on the Persona viewport.
	 * @param InMenuBuilder - The menu builder for the show->clothing menu to extend
	 * @param InPreviewScene - The Persona preview scene from the editor, contains the current preview component
	 */
	virtual void ExtendViewportShowMenu(FMenuBuilder& InMenuBuilder, TSharedRef<IPersonaPreviewScene> InPreviewScene) = 0;

	/**
	 * Called from the editor when clothing is active to process any active debug drawing, recommended to use the
	 * show menu extension for controlling what data to draw
	 * @param InSimulation - The running clothing simulation
	 * @param InOwnerComponent - The component that owns the running clothing simulation
	 * @param PDI - The drawing interface to use
	 */
	UE_API virtual void DebugDrawSimulation(const IClothingSimulationInterface* InSimulation, USkeletalMeshComponent* InOwnerComponent, FPrimitiveDrawInterface* PDI);

	UE_DEPRECATED(5.7, "Use IClothingSimulationInterface instead.")
	virtual void DebugDrawSimulation(const class IClothingSimulation* InSimulation, USkeletalMeshComponent* InOwnerComponent, FPrimitiveDrawInterface* PDI) {}

	/**
	 * Called from the editor when clothing is active to process any active debug drawing of any text strings
	 * Recommended to use the show menu extension for controlling what data to draw
	 * @param InSimulation - The running clothing simulation
	 * @param InOwnerComponent - The component that owns the running clothing simulation
	 * @param Canvas - The canvas to use for drawing the text
	 * @param SceneView - The view on which to project the text
	 */
	UE_API virtual void DebugDrawSimulationTexts(const IClothingSimulationInterface* InSimulation, USkeletalMeshComponent* InOwnerComponent, FCanvas* Canvas, const FSceneView* SceneView);

	UE_DEPRECATED(5.7, "Use IClothingSimulationInterface instead.")
	virtual void DebugDrawSimulationTexts(const class IClothingSimulation* InSimulation, USkeletalMeshComponent* InOwnerComponent, FCanvas* Canvas, const FSceneView* SceneView) {}
};

#undef UE_API
